{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Actor Palette",
	"Description": "Allows creation of Actor Palettes based on existing levels to quickly select actors and drag them into the level editor",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"EnabledByDefault": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ActorPalette",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}