// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ActorPalette : ModuleRules
{
	public ActorPalette(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"InputCore",
				"UnrealEd",
				"ToolMenus",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"ContentBrowser",
				"WorkspaceMenuStructure",
				"DeveloperSettings"
			}
			);
	}
}
