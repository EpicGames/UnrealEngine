// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPalette.h"
#include "ActorPaletteViewport.h"
#include "ActorPaletteViewportClient.h"

//////////////////////////////////////////////////////////////////////
// SActorPalette

void SActorPalette::Construct(const FArguments& InArgs, int32 InTabIndex)
{
	TabIndex = InTabIndex;

	ActorPaletteViewportClient = MakeShareable(new FActorPaletteViewportClient(TabIndex));
	ActorPaletteViewport = SNew(SActorPaletteViewport, ActorPaletteViewportClient, TabIndex);
	ActorPaletteViewportClient->SetOwnerWidget(ActorPaletteViewport);

	ChildSlot
	[
		ActorPaletteViewport.ToSharedRef()
	];
}
