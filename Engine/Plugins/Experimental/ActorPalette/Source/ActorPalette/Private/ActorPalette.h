// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SActorPaletteViewport;
class FActorPaletteViewportClient;

//////////////////////////////////////////////////////////////////////

class SActorPalette : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActorPalette) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, int32 InTabIndex);

private:
	TSharedPtr<SActorPaletteViewport> ActorPaletteViewport;
	TSharedPtr<FActorPaletteViewportClient> ActorPaletteViewportClient;
	int32 TabIndex;
};
