// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteCommands.h"

#define LOCTEXT_NAMESPACE "ActorPalette"

void FActorPaletteCommands::RegisterCommands()
{
	UI_COMMAND(ToggleGameView, "Game View", "Toggles game view.  Game view shows the scene as it appears in game", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::G));
	UI_COMMAND(ResetCameraView, "Reset Camera", "Resets the camera view", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
