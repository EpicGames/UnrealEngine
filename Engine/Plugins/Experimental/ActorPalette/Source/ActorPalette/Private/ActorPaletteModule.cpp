// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteModule.h"
#include "ActorPaletteCommands.h"

#include "Widgets/Docking/SDockTab.h"
#include "Framework/Commands/UICommandList.h"

#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"

#include "ActorPalette.h"

// Core functionality/bugs:
//@TODO: Clear selection state (& invalidate) when drop finished or aborted
//@TODO: Support factorying using actual actor settings (everything but position / rotation, including scale, different material, etc...)
//@TODO: See if we can proper drag-drop behavior (right now the fake drag doesn't fizzle if the mouse is released, and in fact only renders correctly as a preview into the main frame if you release first)
//@TODO: Better workaround (or real fix) for the RenameForPIE hack 

// Important UX stuff:
//@TODO: Save/restore camera position for LRU/favorites
//@TODO: Try doing a timer or something to refresh the non-realtime viewport a few seconds after map load to compensate for texture streaming blurriness 

// Random ideas:
// - Bookmark support (using existing ones, not setting them IMO)
// - Better bookmarks (setting names for example) then showing them as preset buttons on the left side of the viewport
// - Click-drag to pan on LMB that misses meshes
// - Setting for whether or not 'game mode' is enabled by default / remember game mode setting
// - Auto-reload if map being viewed gets saved in the editor
// - Same but for links to other maps, or just create something visual I can double-click on the map itself (or teleport hyperlinks)
// - Support for materials (e.g., detect if it's a material demo kiosk and drag-drop the material instead of the mesh)
// - Store the source map..actor path in package metadata for placed instances and provide a key bind to focus it back (Shift+Ctrl+B maybe?)
// - Add keybinds to cycle between related objects in a set (using metadata on the placed instance linking it back to template map, along with set/chain metadata in the template map or via an associated collection)
//     Should this destroy the existing actor and spawn a new one, only copying transform, or should it do something crazier like try to delta
//     serialize against old template and apply diffs to new template (getting into CPFUO land...)
// - Keybind to randomize Z rotation for selected object (totally unrelated to actor palette, just might be a nice level editor feature?)
// - Picker-style shortcut to let it be used without keeping it up all the time
// - Multi-select (Ctrl+click) when in click but not drag mode?  (unsure if I want to keep it working like it is, see note above in core functionality; could lean on grouping instead if stuff is meant to be placed together)

// Might be bridging too far into foliage / placement tools land here:
// - Support for stamp mode (keep placing instances until you press Escape)
// - Support for optional random z rotation?

#define LOCTEXT_NAMESPACE "ActorPalette"

//////////////////////////////////////////////////////////////////////
// FActorPaletteModule

void FActorPaletteModule::StartupModule()
{
	FActorPaletteStyle::Initialize();

	FActorPaletteCommands::Register();
	
	PluginCommands = MakeShareable(new FUICommandList);

	// Register tab spawners for the actor palette tabs
	const FSlateIcon ActorPaletteIcon(FActorPaletteStyle::GetStyleSetName(), "ActorPalette.TabIcon");
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	TSharedRef<FWorkspaceItem> ActorPaletteGroup = MenuStructure.GetToolsCategory()->AddGroup(
		LOCTEXT("WorkspaceMenu_ActorPaletteCategory", "Actor Palette"),
		LOCTEXT("ActorPaletteMenuTooltipText", "Open an Actor Palette tab."),
		ActorPaletteIcon,
		true);

	for (int32 TabIndex = 0; TabIndex < UE_ARRAY_COUNT(ActorPaletteTabs); TabIndex++)
	{
		const FName TabID = FName(*FString::Printf(TEXT("ActorPaletteTab%d"), TabIndex + 1));
		ActorPaletteTabs[TabIndex].TabID = TabID;

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabID, FOnSpawnTab::CreateRaw(this, &FActorPaletteModule::OnSpawnPluginTab, TabIndex))
			.SetDisplayName(GetActorPaletteLabelWithIndex(TabIndex))
			.SetTooltipText(LOCTEXT("ActorPaletteMenuTooltipText", "Open an Actor Palette tab."))
 			.SetGroup(ActorPaletteGroup)
 			.SetIcon(ActorPaletteIcon);
	}
}

void FActorPaletteModule::ShutdownModule()
{
	FActorPaletteStyle::Shutdown();

	FActorPaletteCommands::Unregister();

	for (int32 TabIndex = 0; TabIndex < UE_ARRAY_COUNT(ActorPaletteTabs); ++TabIndex)
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ActorPaletteTabs[TabIndex].TabID);
	}
}

TSharedRef<SDockTab> FActorPaletteModule::OnSpawnPluginTab(const FSpawnTabArgs& SpawnTabArgs, int32 TabIndex)
{
	TSharedRef<SActorPalette> NewPalette = SNew(SActorPalette, TabIndex);
	
	check(!ActorPaletteTabs[TabIndex].OpenInstance.IsValid());
	ActorPaletteTabs[TabIndex].OpenInstance = NewPalette;

	TAttribute<FText> TabLabel = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &FActorPaletteModule::GetActorPaletteTabLabel, TabIndex));

	TSharedRef<SDockTab> ResultTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(TabLabel)
		[
			NewPalette
		];

	return ResultTab;
}

FText FActorPaletteModule::GetActorPaletteLabelWithIndex(int32 TabIndex)
{
	return FText::Format(LOCTEXT("ActorPaletteTabNameWithIndex", "Actor Palette {0}"), FText::AsNumber(TabIndex + 1));
}

FText FActorPaletteModule::GetActorPaletteTabLabel(int32 TabIndex) const
{
	int32 NumOpenPalettes = 0;
	for (const FActorPaletteTabInfo& TabInfo : ActorPaletteTabs)
	{
		NumOpenPalettes += TabInfo.OpenInstance.IsValid() ? 1 : 0;
	}

	return (NumOpenPalettes > 1) ? GetActorPaletteLabelWithIndex(TabIndex) : LOCTEXT("ActorPaletteTabNoIndex", "Actor Palette");
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FActorPaletteModule, ActorPalette)
