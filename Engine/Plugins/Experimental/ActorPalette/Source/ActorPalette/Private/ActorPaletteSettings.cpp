// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteSettings.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorPaletteSettings)

FAssetData FActorPaletteMapEntry::GetAsAssetData() const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	return AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(MapPath));
}

UActorPaletteSettings::UActorPaletteSettings()
{
	CategoryName = TEXT("Plugins");
}

#if WITH_EDITOR
void UActorPaletteSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	TrimRecentList();
}
#endif

int32 UActorPaletteSettings::FindMapEntry(const FString& DesiredName) const
{
	return SettingsPerLevel.IndexOfByPredicate([&](const FActorPaletteMapEntry& Entry) { return Entry.MapPath == DesiredName; });
}

int32 UActorPaletteSettings::FindLastLevelForTab(int32 TabIndex) const
{
	return MostRecentLevelByTab.IsValidIndex(TabIndex) ? FindMapEntry(MostRecentLevelByTab[TabIndex]) : INDEX_NONE;
}

void UActorPaletteSettings::MarkAsRecentlyUsed(const FAssetData& MapAsset, int32 TabIndex)
{
	if (MapAsset.IsValid())
	{
		const FString MapAssetPath = MapAsset.GetObjectPathString();

		// Remember as the most recent for this tab
		if (!MostRecentLevelByTab.IsValidIndex(TabIndex))
		{
			MostRecentLevelByTab.AddDefaulted(TabIndex + 1 - MostRecentLevelByTab.Num());
		}
		MostRecentLevelByTab[TabIndex] = MapAssetPath;

		// Remember as a recent entry across all tabs, bubbling to the top if is already present
		const int32 RecentIndex = RecentlyUsedList.IndexOfByKey(MapAssetPath);
		if (RecentIndex != INDEX_NONE)
		{
			RecentlyUsedList.Swap(0, RecentIndex);
		}
		else
		{
			RecentlyUsedList.Insert(MapAssetPath, 0);
		}

		// Make sure it's in our list of settings per level
		const int32 SettingsIndex = FindMapEntry(MapAssetPath);
		if (SettingsIndex == INDEX_NONE)
		{
			// New entry
			FActorPaletteMapEntry& NewEntry = SettingsPerLevel[SettingsPerLevel.AddDefaulted()];
			NewEntry.MapPath = MapAsset.GetObjectPathString();
		}
	}

	TrimRecentList();

#if WITH_EDITOR
	PostEditChange();
	SaveConfig();
#endif
}

void UActorPaletteSettings::ToggleFavorite(const FAssetData& MapAsset)
{
	if (FavoritesList.Contains(MapAsset.GetObjectPathString()))
	{
		FavoritesList.Remove(MapAsset.GetObjectPathString());
	}
	else
	{
		FavoritesList.Add(MapAsset.GetObjectPathString());
	}

#if WITH_EDITOR
	PostEditChange();
	SaveConfig();
#endif
}

void UActorPaletteSettings::TrimRecentList()
{
	// Trim the end of the recent list
	const int32 EffectiveLimit = FMath::Max(NumRecentLevelsToKeep, 0);
	while (RecentlyUsedList.Num() > EffectiveLimit)
	{
		RecentlyUsedList.RemoveAt(RecentlyUsedList.Num() - 1);
	}

	// Determine what per-map settings to keep alive
	TSet<FString> InterestingLevels;
	InterestingLevels.Append(RecentlyUsedList);
	InterestingLevels.Append(FavoritesList);
	InterestingLevels.Append(MostRecentLevelByTab);

	// And remove settings that are no longer in any list
	SettingsPerLevel.RemoveAllSwap([&](const FActorPaletteMapEntry& Entry) { return !InterestingLevels.Contains(Entry.MapPath); });
}
