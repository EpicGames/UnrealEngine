// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/Package.h"
#include "ActorPaletteSettings.generated.h"

// Information about a single recent/favorite map
USTRUCT()
struct FActorPaletteMapEntry
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=ActorPalette)
	FString MapPath;

	//@TODO: Store viewpoint

	// Was game mode enabled?
// 	UPROPERTY()
// 	uint8 bGameMode : 1;

	FAssetData GetAsAssetData() const;
};

// Settings/preferences for Actor Palettes
UCLASS(config=EditorPerProjectUserSettings)
class UActorPaletteSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UActorPaletteSettings();

	// Data model note:
	//  - Every tab remembers the last map they had open
	//  - There is a shared recent maps list (opening a map in any tab will bubble it to the top of the recent list)
	//  - There is a shared favorites list
	//  - While a tab is open it has unique viewport settings, but only the most recent user interaction updates the data in the recent/favorites list
	//@TODO: Nothing purges items out of the recent/favorites list if the map is deleted, though recent list is bounded in size

public:
	// Remembered settings for any recent/current/favorite actor palette maps
	UPROPERTY(config)
	TArray<FActorPaletteMapEntry> SettingsPerLevel;

	// List of levels that were recently open in any Actor Palette tab
	UPROPERTY(config)
	TArray<FString> RecentlyUsedList;

	// List of levels that were last open in each Actor Palette tab (indexed by tab index)
	UPROPERTY(config)
	TArray<FString> MostRecentLevelByTab;

	// List of levels that were marked as favorite actor palettes
	UPROPERTY(config)
	TArray<FString> FavoritesList;

	// Should the 'game mode' show flag be set by default for newly opened actor palettes?
	//UPROPERTY(config, EditAnywhere, Category=ActorPalette)
	//bool bEnableGameModeByDefault = true;

	// How many recent levels will be remembered?
	UPROPERTY(config, EditAnywhere, Category=ActorPalette, meta=(ClampMin=0, ClampMax=25))
	int32 NumRecentLevelsToKeep = 10;

#if WITH_EDITOR
	//~UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~End of UObject interface
#endif

	int32 FindMapEntry(const FString& MapName) const;
	int32 FindLastLevelForTab(int32 TabIndex) const;

public:
	void MarkAsRecentlyUsed(const FAssetData& MapAsset, int32 TabIndex);
	void ToggleFavorite(const FAssetData& MapAsset);
	void TrimRecentList();
};

