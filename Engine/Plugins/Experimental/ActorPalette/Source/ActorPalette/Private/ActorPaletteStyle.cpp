// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr< FSlateStyleSet > FActorPaletteStyle::StyleInstance = NULL;

void FActorPaletteStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FActorPaletteStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FActorPaletteStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ActorPaletteStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef< FSlateStyleSet > FActorPaletteStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("ActorPaletteStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("ActorPalette"))->GetBaseDir() / TEXT("Resources"));

	Style->Set("ActorPalette.OpenPluginWindow", new IMAGE_BRUSH(TEXT("ButtonIcon_40x"), Icon40x40));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	Style->Set("ActorPalette.ViewportTitleTextStyle", FTextBlockStyle(NormalText)
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 18))
		.SetColorAndOpacity(FLinearColor(1.0, 1.0f, 1.0f, 0.5f))
		);

	Style->Set("ActorPalette.Palette", new IMAGE_BRUSH(TEXT("Palette_40x"), Icon40x40));
	Style->Set("ActorPalette.Palette.Small", new IMAGE_BRUSH(TEXT("Palette_40x"), Icon20x20));
	Style->Set("ActorPalette.TabIcon", new IMAGE_BRUSH(TEXT("Palette_16x"), Icon16x16));

	Style->Set("ActorPalette.ViewportTitleBackground", new BOX_BRUSH("GraphTitleBackground", FMargin(0)));

	return Style;
}

#undef RootToContentDir

void FActorPaletteStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FActorPaletteStyle::Get()
{
	return *StyleInstance;
}
