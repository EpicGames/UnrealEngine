// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "SEditorViewportToolBarMenu.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#include "ActorPaletteCommands.h"
#include "ActorPaletteViewportClient.h"
#include "ActorPaletteSettings.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "ActorPalette"

//////////////////////////////////////////////////////////////////////////
// SActorPaletteFavoriteEntry

class SActorPaletteFavoriteEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActorPaletteFavoriteEntry) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAssetData& InAssetData, TSharedPtr<FActorPaletteViewportClient> TypedViewportClient)
	{
		AssetData = InAssetData;
		VPC = TypedViewportClient;

		ChildSlot
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ForegroundColor(FSlateColor::UseForeground())
			.OnClicked_Lambda([this]()
			{
				VPC->OpenWorldAsPalette(AssetData);
				FSlateApplication::Get().DismissAllMenus();
				return FReply::Handled();
			})
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.OnClicked_Lambda([this]()
					{
						GetMutableDefault<UActorPaletteSettings>()->ToggleFavorite(AssetData);
						FSlateApplication::Get().DismissAllMenus();
						return FReply::Handled();
					})
					.ToolTipText(LOCTEXT("ToggleFavoriteTooltip", "Mark this level as a favorite or remove it from the favorites list"))
					.ContentPadding(0)
					[
						SNew(SImage)
						.Image_Lambda([this]()
						{
							return FAppStyle::GetBrush(GetDefault<UActorPaletteSettings>()->FavoritesList.Contains(AssetData.GetObjectPathString()) ?
								TEXT("Icons.Star") :
								TEXT("PropertyWindow.Favorites_Disabled"));
						})
					]
				]
				+SHorizontalBox::Slot()
				.Padding(6.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "Menu.Label")
					.Text(FText::Format(LOCTEXT("OpenFavoriteLevel_Label", "{0}"), FText::FromName(AssetData.AssetName)))
					.ToolTipText(FText::Format(LOCTEXT("OpenFavoriteLevel_Tooltip", "Use {0} as an Actor Palette"), FText::FromName(AssetData.PackageName)))
				]
			]
		];
	}

	FAssetData AssetData;
	TSharedPtr<FActorPaletteViewportClient> VPC;
};

//////////////////////////////////////////////////////////////////////////
// SActorPaletteViewportToolbar

// In-viewport toolbar widget used in the actor palette
class SActorPaletteViewportToolbar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SActorPaletteViewportToolbar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider, FOnGetContent&& InSelectMapMenu);

	// SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override;
	virtual void ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const override;
	// End of SCommonEditorViewportToolbarBase

	SActorPaletteViewport& GetOwnerViewport() const;
	FText GetMapMenuLabel() const;
	TSharedRef<SWidget> GenerateMapMenu() const;

	FOnGetContent SelectMapMenuCallback;
};

SActorPaletteViewport& SActorPaletteViewportToolbar::GetOwnerViewport() const
{
	return StaticCastSharedRef<SActorPaletteViewport, SEditorViewport>(GetInfoProvider().GetViewportWidget()).Get();
}

void SActorPaletteViewportToolbar::Construct(const FArguments& InArgs, TSharedPtr<class ICommonEditorViewportToolbarInfoProvider> InInfoProvider, FOnGetContent&& InSelectMapMenu)
{
	SelectMapMenuCallback = MoveTemp(InSelectMapMenu);
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments(), InInfoProvider);
}

TSharedRef<SWidget> SActorPaletteViewportToolbar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddMenuEntry(FActorPaletteCommands::Get().ToggleGameView);
		ShowMenuBuilder.AddMenuEntry(FActorPaletteCommands::Get().ResetCameraView);
	}

	return ShowMenuBuilder.MakeWidget();
}

void SActorPaletteViewportToolbar::ExtendLeftAlignedToolbarSlots(TSharedPtr<SHorizontalBox> MainBoxPtr, TSharedPtr<SViewportToolBar> ParentToolBarPtr) const
{
	const FMargin ToolbarSlotPadding(2.0f, 2.0f);

	MainBoxPtr->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.Label(LOCTEXT("MapMenu_Label", "Choose Level"))
			.OnGetMenuContent(SelectMapMenuCallback)
			.Cursor(EMouseCursor::Default)
			.ParentToolBar(ParentToolBarPtr)
		];
}

//////////////////////////////////////////////////////////////////////////
// SActorPaletteViewport

SActorPaletteViewport::~SActorPaletteViewport()
{
	TypedViewportClient.Reset();
}

void SActorPaletteViewport::Construct(const FArguments& InArgs, TSharedPtr<class FActorPaletteViewportClient> InViewportClient, int32 InTabIndex)
{
	TypedViewportClient = InViewportClient;
	TabIndex = InTabIndex;

	SEditorViewport::Construct(SEditorViewport::FArguments());

	TSharedRef<SOverlay> MyOverlay = SNew(SOverlay)
		+SOverlay::Slot()
		[
			ChildSlot.GetWidget()
		];

	// Show quick buttons to open / choose whenever there is no map open
	MyOverlay->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility_Lambda([this]() { return TypedViewportClient->GetCurrentWorldAssetData().IsValid() ? EVisibility::Collapsed : EVisibility::Visible; })

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 32.0f, 0.0f)
			.VAlign(VAlign_Fill)
			[
				SNew(SButton)
				.Text(LOCTEXT("QuickButtonLabel_ReloadLastMap", "Open Last Level"))
				.VAlign(VAlign_Center)
				.Visibility_Lambda([this]()
				{
					return (GetDefault<UActorPaletteSettings>()->FindLastLevelForTab(TabIndex) != INDEX_NONE) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.ToolTipText_Lambda([this]()
				{
					const UActorPaletteSettings* Settings = GetDefault<UActorPaletteSettings>();
					const int32 LastLevelForTabIndex = Settings->FindLastLevelForTab(TabIndex);
					return (LastLevelForTabIndex != INDEX_NONE) ?
						FText::Format(LOCTEXT("QuickButtonTooltip_ReloadLastMap", "Use {0} as an Actor Palette"), FText::FromName(Settings->SettingsPerLevel[LastLevelForTabIndex].GetAsAssetData().PackageName)) :
						FText::GetEmpty();
				})
				.OnClicked_Lambda([this]()
				{
					const UActorPaletteSettings* Settings = GetDefault<UActorPaletteSettings>();
					int32 LastLevelIndex = Settings->FindLastLevelForTab(TabIndex);
					if (LastLevelIndex != INDEX_NONE)
					{
						TypedViewportClient->OpenWorldAsPalette(Settings->SettingsPerLevel[LastLevelIndex].GetAsAssetData());
					}
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &SActorPaletteViewport::GenerateMapMenu)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("QuickButtonLabel_ChooseMap", "Choose Level"))
					.ToolTipText(LOCTEXT("QuickButtonTooltip_ChooseMap", "Choose a Level to use as an Actor Palette"))
				]
			]
		];

	// Show the name of the currently open map
	MyOverlay->AddSlot()
		.VAlign(VAlign_Bottom)
		[
			SNew(SBorder)
			.BorderImage(FActorPaletteStyle::Get().GetBrush("ActorPalette.ViewportTitleBackground"))
			.HAlign(HAlign_Fill)
			.Visibility(EVisibility::HitTestInvisible)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SScaleBox)
					.HAlign(HAlign_Center)
					.StretchDirection(EStretchDirection::DownOnly)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(STextBlock)
						.TextStyle(FActorPaletteStyle::Get(), "ActorPalette.ViewportTitleTextStyle")
						.Text(this, &SActorPaletteViewport::GetTitleText)
					]
				]
			]
		];

	this->ChildSlot
	[
		MyOverlay
	];
}

TSharedPtr<SWidget> SActorPaletteViewport::BuildViewportToolbar()
{
	return SNew(SActorPaletteViewportToolbar, SharedThis(this), FOnGetContent::CreateSP(this, &SActorPaletteViewport::GenerateMapMenu));
}

TSharedRef<FEditorViewportClient> SActorPaletteViewport::MakeEditorViewportClient()
{
	return TypedViewportClient.ToSharedRef();
}

void SActorPaletteViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	FActorPaletteCommands::Register();
 	const FActorPaletteCommands& Commands = FActorPaletteCommands::Get();

	CommandList->MapAction(
		Commands.ToggleGameView,
		FExecuteAction::CreateLambda([this]() { TypedViewportClient->SetGameView(!TypedViewportClient->IsInGameView()); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this](){ return TypedViewportClient->IsInGameView(); }));

	CommandList->MapAction(
		Commands.ResetCameraView,
		FExecuteAction::CreateLambda([this]() { TypedViewportClient->ResetCameraView(); }));
}

void SActorPaletteViewport::OnFocusViewportToSelection()
{
//@TODO:
//	TypedViewportClient->RequestFocusOnSelection(/*bInstant=*/ false);
}

TSharedRef<class SEditorViewport> SActorPaletteViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SActorPaletteViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SActorPaletteViewport::OnFloatingButtonClicked()
{
}

FText SActorPaletteViewport::GetTitleText() const
{
	const FName CurrentMapName = TypedViewportClient->GetCurrentWorldAssetData().PackageName;
	if (CurrentMapName == NAME_None)
	{
		return LOCTEXT("ActorPaletteViewportTitle_NoMap", "Choose a level to use as a palette");
	}
	else
	{
		return FText::Format(LOCTEXT("ActorPaletteViewportTitle_MapName", "{0}"), FText::FromName(CurrentMapName));
	}
}

TSharedRef<SWidget> SActorPaletteViewport::GenerateMapMenu() const
{
	const FActorPaletteCommands& Actions = FActorPaletteCommands::Get();
	UActorPaletteSettings* Settings = GetMutableDefault<UActorPaletteSettings>();

	TSharedPtr<FExtender> MenuExtender = GetExtenders();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder InMenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList, MenuExtender);

	InMenuBuilder.PushCommandList(CommandList.ToSharedRef());
	if (MenuExtender.IsValid())
	{ 
		InMenuBuilder.PushExtender(MenuExtender.ToSharedRef());
	}

	// Add an entry to chose an arbitrary map via an asset picker
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("OpenLevelFromPicker_Label", "Open Level..."),
			LOCTEXT("OpenLevelFromPicker_Tooltip", "Select an existing level asset to use as an Actor Palette"),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
							
				// Configure filter for asset picker
				FAssetPickerConfig Config;
				Config.Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
				Config.InitialAssetViewType = EAssetViewType::List;
				Config.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& AssetData)
				{
					TypedViewportClient->OpenWorldAsPalette(AssetData);
					FSlateApplication::Get().DismissAllMenus();
				});
				Config.bAllowDragging = false;
				Config.bAllowNullSelection = true;
				Config.bFocusSearchBoxWhenOpened = true;
				Config.InitialAssetSelection = TypedViewportClient->GetCurrentWorldAssetData();

				TSharedRef<SWidget> ChooseMapWidget =
					SNew(SBox)
					.WidthOverride(300.f)
					.HeightOverride(300.f)
					[
						ContentBrowserModule.Get().CreateAssetPicker(Config)
					];

				SubMenuBuilder.BeginSection("Browse", LOCTEXT("BrowseHeader", "Browse"));
				SubMenuBuilder.AddWidget(ChooseMapWidget, FText::GetEmpty());
				SubMenuBuilder.EndSection();
			})
		);
	}

	// Add an entry for the first few selected maps from the content browser (ala a 'Use' arrow button)
	{
		TArray<FAssetData> SelectedAssets;
		GEditor->GetContentBrowserSelections(/*out*/ SelectedAssets);

		int32 NumLeftAllowedFromContentBrowser = 4;
		for (FAssetData& Asset : SelectedAssets)
		{
			if (Asset.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
			{
				FUIAction Action;
				Action.ExecuteAction.BindLambda([this, Asset]()
				{
					TypedViewportClient->OpenWorldAsPalette(Asset);
				});

				InMenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("OpenSelectedLevelFromCB_Label", "Open {0} (from Content Browser)"), FText::FromName(Asset.AssetName)),
					FText::Format(LOCTEXT("OpenSelectedLevelFromCB_Tooltip", "Use {0} as an Actor Palette"), FText::FromName(Asset.PackageName)),
					FSlateIcon(),
					Action);

				--NumLeftAllowedFromContentBrowser;
				if (NumLeftAllowedFromContentBrowser == 0)
				{
					break;
				}
			}
		}
	}
			
	// Add an entry to chose a recent map
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("OpenLevelFromRecentList_Label", "Recent Levels"),
			LOCTEXT("OpenLevelFromRecentList_Tooltip", "Select a level recently used as an Actor Palette"),
			FNewMenuDelegate::CreateLambda([this, Settings](FMenuBuilder& SubMenuBuilder)
			{
				bool bAddedAnyRecent = false;
				for (const FString& RecentEntry : Settings->RecentlyUsedList)
				{
					const int32 SettingsIndex = Settings->FindMapEntry(RecentEntry);
					if (ensure(SettingsIndex != INDEX_NONE))
					{
						FAssetData Asset = Settings->SettingsPerLevel[SettingsIndex].GetAsAssetData();
						bAddedAnyRecent = true;

						FUIAction Action;
						Action.ExecuteAction.BindLambda([this, Asset]()
						{
							TypedViewportClient->OpenWorldAsPalette(Asset);
						});

						SubMenuBuilder.AddMenuEntry(
							FText::Format(LOCTEXT("OpenRecentLevel_Label", "{0}"), FText::FromName(Asset.AssetName)),
							FText::Format(LOCTEXT("OpenRecentLevel_Tooltip", "Use {0} as an Actor Palette"), FText::FromName(Asset.PackageName)),
							FSlateIcon(),
							Action);
					}
				}

				if (!bAddedAnyRecent)
				{
					const FText NoRecents = LOCTEXT("NoRecents_Label", "There are no recent levels");
					SubMenuBuilder.AddMenuEntry(NoRecents, NoRecents, FSlateIcon(), FUIAction());
				}
			})
		);
	}

	// Add an entry to chose a favorite map
	{
		InMenuBuilder.AddSubMenu(
			LOCTEXT("OpenLevelFromFavoriteList_Label", "Favorite Levels"),
			LOCTEXT("OpenLevelFromFavoriteList_Tooltip", "Select a favorite level as an Actor Palette"),
			FNewMenuDelegate::CreateLambda([this, Settings](FMenuBuilder& SubMenuBuilder)
			{
				if (TypedViewportClient->GetCurrentWorldAssetData().IsValid())
				{
					SubMenuBuilder.BeginSection(NAME_None, LOCTEXT("FavoritesMenu_CurrentMap", "Current Level"));
					SubMenuBuilder.AddWidget(SNew(SActorPaletteFavoriteEntry, TypedViewportClient->GetCurrentWorldAssetData(), TypedViewportClient), FText::GetEmpty());
					SubMenuBuilder.EndSection();
				}

				SubMenuBuilder.BeginSection(NAME_None, LOCTEXT("FavoritesMenu_FavoritesList", "Favorites"));

				bool bAddedAnyFavorites = false;
				for (const FString& FavoriteEntry : Settings->FavoritesList)
				{
					const int32 SettingsIndex = Settings->FindMapEntry(FavoriteEntry);
					if (ensure(SettingsIndex != INDEX_NONE))
					{
						FAssetData Asset = Settings->SettingsPerLevel[SettingsIndex].GetAsAssetData();
						bAddedAnyFavorites = true;

						SubMenuBuilder.AddWidget(SNew(SActorPaletteFavoriteEntry, Asset, TypedViewportClient), FText::GetEmpty());
					}
				}

				if (!bAddedAnyFavorites)
				{
					const FText NoFavorites = LOCTEXT("NoFavorites_Label", "There are no favorite levels");
					SubMenuBuilder.AddMenuEntry(NoFavorites, NoFavorites, FSlateIcon(), FUIAction());
				}

				SubMenuBuilder.EndSection();
			})
		);
	}

	InMenuBuilder.PopCommandList();
	if (MenuExtender.IsValid())
	{
		InMenuBuilder.PopExtender();
	}

	return InMenuBuilder.MakeWidget();
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
