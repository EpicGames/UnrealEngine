// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"

class FActorPaletteViewportClient;
class SActorPaletteViewportToolbar;

//////////////////////////////////////////////////////////////////////////
// SActorPaletteViewport

class SActorPaletteViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(SActorPaletteViewport) {}
	SLATE_END_ARGS()

	~SActorPaletteViewport();

	void Construct(const FArguments& InArgs, TSharedPtr<FActorPaletteViewportClient> InViewportClient, int32 InTabIndex);
//	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	// SEditorViewport interface
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual void BindCommands() override;
	virtual void OnFocusViewportToSelection() override;
	// End of SEditorViewport interface

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

protected:
	FText GetTitleText() const;
	TSharedRef<SWidget> GenerateMapMenu() const;

private:
	TSharedPtr<FActorPaletteViewportClient> TypedViewportClient;
	int32 TabIndex;

	friend SActorPaletteViewportToolbar;
};
