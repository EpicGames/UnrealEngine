// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorPaletteViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "EngineUtils.h"
#include "Engine/LevelStreamingDynamic.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "InputKeyEventArgs.h"
#include "MouseDeltaTracker.h"
#include "ActorPaletteSettings.h"
#include "Misc/ScopedSlowTask.h"
#include "Selection.h"
#include "Widgets/SWindow.h"
#include "SEditorViewport.h"

#define LOCTEXT_NAMESPACE "ActorPalette"

//////////////////////////////////////////////////////////////////////////
// FActorPaletteViewportClient

FActorPaletteViewportClient::FActorPaletteViewportClient(int32 InTabIndex)
	: FEditorViewportClient(nullptr, nullptr, nullptr)
	, TabIndex(InTabIndex)
{
	SetViewModes(VMI_Lit, VMI_Lit);

	// Get the correct general direction of the perspective mode; the distance doesn't matter much as we've queued up a deferred zoom that will calculate a much better distance
	//@TODO: Save/load camera views into settings... (SetInitialViewTransform + SetViewportType)

	MyBGColor = FLinearColor::Black;

	SetRealtime(false);

	PreviewScene = &OwnedPreviewScene;

	EngineShowFlags.Selection = true;
	EngineShowFlags.SelectionOutline = true;
	EngineShowFlags.Grid = false;
}

bool FActorPaletteViewportClient::InputKey(const FInputKeyEventArgs& InEventArgs)
{
	if (InEventArgs.Key == EKeys::LeftMouseButton)
	{
		if (FSlateApplication::Get().IsDragDropping())
		{
// 			if (InEventArgs.Event == IE_Released)
// 			{
// 				FSlateApplication::Get().CancelDragDrop();
// 			}

			return true;
		}

		const int32	HitX = InEventArgs.Viewport->GetMouseX();
		const int32	HitY = InEventArgs.Viewport->GetMouseY();

		// Calc the raw delta from the mouse to detect if there was any movement
		FVector RawMouseDelta = MouseDeltaTracker->GetRawDelta();

		// Note: We are using raw mouse movement to double check distance moved in low performance situations.  In low performance situations its possible
		// that we would get a mouse down and a mouse up before the next tick where GEditor->MouseMovment has not been updated.
		// In that situation, legitimate drags are incorrectly considered clicks
		bool bNoMouseMovment = RawMouseDelta.SizeSquared() < MOUSE_CLICK_DRAG_DELTA && GEditor->MouseMovement.SizeSquared() < MOUSE_CLICK_DRAG_DELTA;

		if (bNoMouseMovment && !MouseDeltaTracker->WasExternalMovement())
		{
			// If the mouse haven't moved too far, treat the button release as a click.
		}
		else
		{
			HHitProxy* HitProxy = InEventArgs.Viewport->GetHitProxy(HitX, HitY);


			if (HActor* ActorProxy = HitProxyCast<HActor>(HitProxy))
			{
				if ((ActorProxy->Actor != nullptr) && !ActorProxy->Actor->IsLockLocation())
				{
					TArray<UObject*> Assets;
					ActorProxy->Actor->GetReferencedContentObjects(Assets);

					if ((Assets.Num() > 0) && (Assets[0] != nullptr))
					{
						USelection* CBSelection = GEditor->GetSelectedActors();
						CBSelection->Select(ActorProxy->Actor, true);
						ActorProxy->Actor->MarkComponentsRenderStateDirty();
						Invalidate();
						FSlateApplication::Get().CancelDragDrop();

						const FVector2D CurrentCursorPosition = FSlateApplication::Get().GetCursorPos();
						const FVector2D LastCursorPosition = FSlateApplication::Get().GetLastCursorPos();

						TSharedPtr<FAssetDragDropOp> DragDropOperation = FAssetDragDropOp::New(FAssetData(Assets[0], true));

						TSet<FKey> PressedMouseButtons;
						PressedMouseButtons.Add(InEventArgs.Key);

						FModifierKeysState ModifierKeyState;

						FPointerEvent FakePointerEvent(
							FSlateApplication::Get().GetUserIndexForMouse(),
							FSlateApplicationBase::CursorPointerIndex,
							CurrentCursorPosition,
							LastCursorPosition,
							PressedMouseButtons,
							EKeys::Invalid,
							0,
							ModifierKeyState);

						// Tell slate to enter drag and drop mode.
						// Make a faux mouse event for slate, so we can initiate a drag and drop.
						FDragDropEvent DragDropEvent(FakePointerEvent, DragDropOperation);

						TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(GetEditorViewportWidget().ToSharedRef());

						FSlateApplication::Get().ProcessDragEnterEvent(OwnerWindow.ToSharedRef(), DragDropEvent);
					}
				}
			}

		}

		return true;
	}
	else
	{
		FInputKeyEventArgs Args = InEventArgs;
		Args.Key = (InEventArgs.Key == EKeys::RightMouseButton) ? EKeys::LeftMouseButton : InEventArgs.Key;
		return FEditorViewportClient::InputKey(Args);
	}
}

FLinearColor FActorPaletteViewportClient::GetBackgroundColor() const
{
	return MyBGColor;
}

void FActorPaletteViewportClient::Tick(float DeltaSeconds) 
{
	FEditorViewportClient::Tick(DeltaSeconds);

	if ( this->IsRealtime() )
	{
		OwnedPreviewScene.GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
	}
}

void FActorPaletteViewportClient::OpenWorldAsPalette(const FAssetData& InSourceWorldAsset)
{
	FScopedSlowTask SlowTask(0.0f, LOCTEXT("LoadingLevelAsPalette", "Loading Level into Actor Palette..."));

	UWorld* TargetWorld = OwnedPreviewScene.GetWorld();

	if (CurrentLevelStreaming != nullptr)
	{
		CurrentLevelStreaming->SetIsRequestingUnloadAndRemoval(true);
		TargetWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		CurrentLevelStreaming = nullptr;
	}

	// Copy if the source world is valid
	SourceWorldAsset = InSourceWorldAsset;
	if (UWorld* SourceWorld = Cast<UWorld>(SourceWorldAsset.GetAsset()))
	{
		bool bSucceeded;
		ULevelStreamingDynamic* NewLevel = ULevelStreamingDynamic::LoadLevelInstance(TargetWorld, SourceWorld->GetPathName(), FVector::ZeroVector, FRotator::ZeroRotator, /*out*/ bSucceeded);

		if (bSucceeded)
		{
			//@TODO: This is a squiffy workaround for ULevelStreamingDynamic::LoadLevelInstance doing the wrong thing if the level is already loaded and we're not in PIE
			// (this doesn't make it go down the code path that makes it work in PIE, it just guarantees the destination name isn't the same as the asset name anymore...)
			NewLevel->RenameForPIE(1);

			CurrentLevelStreaming = NewLevel;

			TargetWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);

			TargetWorld->EditorViews = SourceWorld->EditorViews;

			ResetCameraView();
		}
	}

	// Update the most-recently used list
	UActorPaletteSettings* Settings = GetMutableDefault<UActorPaletteSettings>();
	Settings->MarkAsRecentlyUsed(SourceWorldAsset, TabIndex);

	// Redraw the viewport
	Invalidate();
}

void FActorPaletteViewportClient::ResetCameraView()
{
	UWorld* World = OwnedPreviewScene.GetWorld();

	if (World->EditorViews.IsValidIndex(GetViewportType()))
	{
		FLevelViewportInfo& ViewportInfo = World->EditorViews[GetViewportType()];
		SetInitialViewTransform(
			GetViewportType(),
			ViewportInfo.CamPosition,
			ViewportInfo.CamRotation,
			ViewportInfo.CamOrthoZoom);
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
