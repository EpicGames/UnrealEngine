// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewScene.h"
#include "EditorViewportClient.h"

class FCanvas;
class FScopedTransaction;
class FUICommandList;
class ULevelStreamingDynamic;

//////////////////////////////////////////////////////////////////////////
// FActorPaletteViewportClient

class FActorPaletteViewportClient : public FEditorViewportClient
{
public:
	FActorPaletteViewportClient(int32 InTabIndex);

	// FViewportClient interface
	virtual bool InputKey(const FInputKeyEventArgs& InEventArgs) override;
	// End of FViewportClient interface

	// FEditorViewportClient interface
	virtual FLinearColor GetBackgroundColor() const override;
	void Tick(float DeltaSeconds) override;
	// End of FEditorViewportClient interface

	void OpenWorldAsPalette(const FAssetData& SourceWorldAsset);
	FAssetData GetCurrentWorldAssetData() const { return SourceWorldAsset; }

	void SetOwnerWidget(const TWeakPtr<SEditorViewport> OwnerPtr)
	{
		EditorViewportWidget = OwnerPtr;
	}

	void ResetCameraView();

private:
	int32 TabIndex;

	// The preview scene
	FPreviewScene OwnedPreviewScene;

	FAssetData SourceWorldAsset;

	ULevelStreamingDynamic* CurrentLevelStreaming = nullptr;

	FLinearColor MyBGColor;
};
