// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "ActorPaletteStyle.h"

class FActorPaletteCommands : public TCommands<FActorPaletteCommands>
{
public:

	FActorPaletteCommands()
		: TCommands<FActorPaletteCommands>(TEXT("ActorPalette"), NSLOCTEXT("Contexts", "ActorPalette", "ActorPalette Plugin"), NAME_None, FActorPaletteStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ToggleGameView;
	TSharedPtr<FUICommandInfo> ResetCameraView;
};