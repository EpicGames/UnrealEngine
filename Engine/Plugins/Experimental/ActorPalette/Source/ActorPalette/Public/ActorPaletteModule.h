// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class SActorPalette;

#define MAX_ACTOR_PALETTES 4

class FActorPaletteModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** This function will be bound to Command (by default it will bring up plugin window) */
	void PluginButtonClicked();
	
private:
	TSharedRef<class SDockTab> OnSpawnPluginTab(const class FSpawnTabArgs& SpawnTabArgs, int32 TabIndex);

	static FText GetActorPaletteLabelWithIndex(int32 TabIndex);
	FText GetActorPaletteTabLabel(int32 TabIndex) const;

private:
	struct FActorPaletteTabInfo
	{
		FName TabID;
		TWeakPtr<SActorPalette> OpenInstance;
	};

	FActorPaletteTabInfo ActorPaletteTabs[MAX_ACTOR_PALETTES];
	TSharedPtr<class FUICommandList> PluginCommands;
};
